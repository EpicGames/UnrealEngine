// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExecuteConsoleCommand.h"
#include "ISessionServicesModule.h"
#include "UnrealFrontendMain.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Ticker.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/CommandLine.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Modules/ModuleManager.h"

void FExecuteConsoleCommand::Run(const FString& Params)
{
	FString TargetInstanceIdStr = {};
	FGuid TargetInstanceId;
	if (!FParse::Value(FCommandLine::Get(), TEXT("-InstanceId="), TargetInstanceIdStr) || TargetInstanceIdStr.IsEmpty())
	{
		UE_LOGF(LogUFECommands, Warning, "InstanceId was not set. Please use '-InstanceId=' in your command line. Tip: -InstanceId= can also be passed to any unreal application to override it's instance id.");
		return;
	}
	if (!FGuid::Parse(TargetInstanceIdStr, TargetInstanceId))
	{
		UE_LOGF(LogUFECommands, Warning, "Unable to parse InstanceId FGuid.");
		return;
	}

	FString TargetCommand;
	if (!FParse::Value(FCommandLine::Get(), TEXT("-Command="), TargetCommand) || TargetCommand.IsEmpty())
	{
		UE_LOGF(LogUFECommands, Warning, "Command was not found. Please use '-Command=' in your command line.");
		return;
	}

	double Timeout = 30.0;
	FParse::Value(FCommandLine::Get(), TEXT("-Timeout="), Timeout);

	// UdpMessaging will not start it's thread without correct loading phase
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PostDefault);

	FModuleManager::Get().LoadModuleChecked("UdpMessaging");

	TSharedPtr<ISessionManager> SessionManager = FModuleManager::Get().LoadModuleChecked<ISessionServicesModule>("SessionServices").GetSessionManager();

	double Start = FPlatformTime::Seconds();

	TSet<FGuid> DiscoveredInstanceIds;
	bool bTargetCommandExecuted = false;

	while (!IsEngineExitRequested() && (FPlatformTime::Seconds() - Start < Timeout) && !bTargetCommandExecuted)
	{
		TArray<TSharedPtr<ISessionInfo>> Sessions;
		SessionManager->GetSessions(Sessions);
		for (TSharedPtr<ISessionInfo> Session : Sessions)
		{
			TArray<TSharedPtr<ISessionInstanceInfo>> Instances;
			Session->GetInstances(Instances);

			for (TSharedPtr<ISessionInstanceInfo> Instance : Instances)
			{
				const FGuid& InstanceId = Instance->GetInstanceId();
				const FString InstanceIdStr = InstanceId.ToString(EGuidFormats::DigitsWithHyphensLower);
				if (!DiscoveredInstanceIds.Contains(InstanceId))
				{
					UE_LOGF(LogUFECommands, Display, "Discovered InstanceId:%ls InstanceName:%ls Owner:%ls SessionId:%ls",
						*InstanceIdStr,
						*Instance->GetInstanceName(),
						*Session->GetSessionOwner(),
						*Session->GetSessionId().ToString(EGuidFormats::DigitsWithHyphensLower));

					DiscoveredInstanceIds.Add(InstanceId);
				}

				if (Instance->GetEngineVersion() == 0) // wait until instance has responded
				{
					continue;
				}

				if (InstanceId == TargetInstanceId)
				{
					if (Instance->IsAuthorized())
					{
						Instance->ExecuteCommand(TargetCommand);
						UE_LOGF(LogUFECommands, Warning, "Executed '%ls' on InstanceId:%ls", *TargetCommand, *InstanceIdStr);
					}
					else
					{
						UE_LOGF(LogUFECommands, Warning, "Not authorized to interact with InstanceId:%ls", *InstanceIdStr);
					}

					bTargetCommandExecuted = true;
					break;
				}
			}
		}

		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread_Local);

		FTSTicker::GetCoreTicker().Tick(0.1f);

		FPlatformProcess::Sleep(0.1f);

		GLog->FlushThreadedLogs();
	}
}
