// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ISessionInstanceInfo;
class ISessionInfo;
class ISessionManager;

class FExecuteConsoleCommand
{
public:
	/**
	* Executes the command.
	* Use '-Command=' command line argument to specify the console command.
	* Use '-InstanceId=' command line argument to specify the target instance id.
	*/
	void Run(const FString& Params);
};
