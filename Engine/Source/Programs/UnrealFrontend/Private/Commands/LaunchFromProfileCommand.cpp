// Copyright Epic Games, Inc. All Rights Reserved.

#include "LaunchFromProfileCommand.h"
#include "DesktopPlatformModule.h"
#include "ILauncherProfile.h"
#include "ILauncherProfileManager.h"
#include "ILauncherTask.h"
#include "ILauncherWorker.h"
#include "ILauncher.h"
#include "ILauncherServicesModule.h"
#include "ITargetDeviceServicesModule.h"
#include "UnrealFrontendMain.h"
#include "Misc/CommandLine.h"
#include "Misc/OutputDeviceRedirector.h"

void FLaunchFromProfileCommand::Run(const FString& Params)
{
	// Get the name of the profile from the command line.
	FString ProfileName;
	FParse::Value(FCommandLine::Get(), TEXT("-PROFILENAME="), ProfileName);
	if (ProfileName.IsEmpty())
	{
		UE_LOGF(LogUFECommands, Warning, "Profile name was found.  Please use '-PROFILENAME=' in your command line.");
		return;
	}
	
	// Loading the launcher services module to get the needed profile.
	ILauncherServicesModule& LauncherServicesModule = FModuleManager::LoadModuleChecked<ILauncherServicesModule>(TEXT("LauncherServices"));
	ILauncherProfileManagerRef ProfileManager = LauncherServicesModule.GetProfileManager();
	ILauncherProfilePtr Profile = ProfileManager->FindProfile(ProfileName);

	// Loading the Device Proxy Manager to get the needed Device Manager.
	ITargetDeviceServicesModule& DeviceServiceModule = FModuleManager::LoadModuleChecked<ITargetDeviceServicesModule>(TEXT("TargetDeviceServices"));
	TSharedRef<ITargetDeviceProxyManager> DeviceProxyManager = DeviceServiceModule.GetDeviceProxyManager();

	UE_LOGF(LogUFECommands, Display, "Begin the process of launching a project using the provided profile.");
	ILauncherRef LauncherRef = LauncherServicesModule.CreateLauncher();
	ILauncherWorkerPtr LauncherWorkerPtr = LauncherRef->Launch(DeviceProxyManager, Profile.ToSharedRef());

	// This will allow us to pipe the launcher messages into the command window.
	LauncherWorkerPtr.Get()->OnOutputReceived().AddStatic(&FLaunchFromProfileCommand::MessageReceived);
	// Allows us to exit this command once the launcher worker has completed or is canceled 
	LauncherWorkerPtr.Get()->OnCompleted().AddRaw(this, &FLaunchFromProfileCommand::LaunchCompleted);
	LauncherWorkerPtr.Get()->OnCanceled().AddRaw(this, &FLaunchFromProfileCommand::LaunchCanceled);

	TArray<ILauncherTaskPtr> TaskList;
	int32 NumOfTasks = LauncherWorkerPtr->GetTasks(TaskList);	
	UE_LOGF(LogUFECommands, Display, "There are '%i' tasks to be completed.", NumOfTasks);

	// Holds the current element in the TaskList array.
	int32 TaskIndex = 0;
	// Holds the name of the current tasked.
	FString TriggeredTask;

	bTestRunning = true;
	while (bTestRunning)
	{
		if (TaskIndex >= NumOfTasks) continue;
		ILauncherTaskPtr CurrentTask = TaskList[TaskIndex];

		// Log the current task, but only once per run.
		if (CurrentTask->GetStatus() == ELauncherTaskStatus::Busy)
		{
			if (CurrentTask->GetDesc() == TriggeredTask) continue;

			TriggeredTask = *CurrentTask->GetDesc();
			UE_LOGF(LogUFECommands, Display, "Current Task is %ls", *TriggeredTask);
			TaskIndex++;
		}
	}
}

void FLaunchFromProfileCommand::MessageReceived(const FString& InMessage)
{
	GLog->Logf(ELogVerbosity::Log, TEXT("%s"), *InMessage);
}

void FLaunchFromProfileCommand::LaunchCompleted(bool Outcome, double ExecutionTime, int32 ReturnCode)
{
	UE_LOGF(LogUFECommands, Log, "Profile launch command %ls.", Outcome ? TEXT("is SUCCESSFUL") : TEXT("has FAILED"));
	bTestRunning = false;
}

void FLaunchFromProfileCommand::LaunchCanceled(double ExecutionTime)
{
	UE_LOGF(LogUFECommands, Log, "Profile launch command was canceled.");
	bTestRunning = false;
}