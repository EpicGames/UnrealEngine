// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FLaunchFromProfileCommand
{
public:

	/**
	* Executes the command.
	* Use '-PROFILENAME=' command line argument to specify the name of a profile to use for this command.
	*/
	void Run(const FString& Params);

protected:

	/**
	* Logs out the messages from the launcher worker.  Used in a delegate.
	*/
	static void MessageReceived(const FString& InMessage);

	void LaunchCompleted(bool Outcome, double ExecutionTime, int32 ReturnCode);
	void LaunchCanceled(double ExecutionTime);

private:
	bool bTestRunning;
};
