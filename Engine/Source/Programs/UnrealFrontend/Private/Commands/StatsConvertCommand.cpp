// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatsConvertCommand.h"

#if STATS && UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8

#include "Misc/CommandLine.h"
#include "HAL/FileManager.h"
#include "Templates/UniquePtr.h"

/** Helper class used to extract stats data into CSV file. */
class FCSVStatsProfiler : public FStatsReadFile
{
	friend struct FStatsReader<FCSVStatsProfiler>;
	typedef FStatsReadFile Super;

protected:

	/** Initialization constructor. */
	FCSVStatsProfiler( const TCHAR* InFilename )
		: FStatsReadFile( InFilename, false )
		, CSVWriter( nullptr )
	{
		// Keep only the last frame.
		SetHistoryFrames( 1 );
	}

	/** Called every each frame has been read from the file. */
	virtual void ReadStatsFrame( const TArray<FStatMessage>& CondensedMessages, const int64 Frame ) override;

	/** Writes a formatted string to the CSV file. */
	void WriteString( const TCHAR* Format, ... );
public:

	/** Sets a writer used to serialize the data for the CSV file. */
	void Initialize( FArchive* InCSVWriter, const TArray<FString>& StatArrayString )
	{
		CSVWriter = InCSVWriter;

		// Find the raw names for faster compare.
		for (const FString& It : StatArrayString)
		{
			// Check for the short name.
			const FStatMessage* LongNamePtr = State.ShortNameToLongName.Find( *It );
			if (LongNamePtr != nullptr)
			{
				StatRawNames.Add( LongNamePtr->NameAndInfo.GetRawName() );
				StatShortNames.Add( LongNamePtr->NameAndInfo.GetShortName() );
			}
		}

		// Output the csv header.
		WriteString( TEXT("Frame,Name,Value\r\n") );
	}

protected:
	FArchive* CSVWriter;
	/** Stats' raw names for fast comparison. */
	TArray<FName> StatRawNames;

	/** Stats' short names for writing into the CSV file. */
	TArray<FName> StatShortNames;
};

void FCSVStatsProfiler::ReadStatsFrame( const TArray<FStatMessage>& CondensedMessages, const int64 Frame )
{
	// get the thread stats
	TArray<FStatMessage> Stats;
	State.GetInclusiveAggregateStackStats( CondensedMessages, Stats );

	// The tick rate for different platforms will be different. We cannot use the Windows tick rate accurately here. We will pull it from the stats,
	// but until we encounter one our best bet is to leave values as full ticks that can be analysed by hand.
	double MillisecondsPerCycle = 1.0f;

	for (int32 Index = 0; Index < Stats.Num(); ++Index)
	{
		const FStatMessage& StatMessage = Stats[Index];
		//UE_LOGF(LogTemp, Display, "Stat: %ls", *Meta.NameAndInfo.GetShortName().ToString());

		if (StatMessage.NameAndInfo.GetRawName() == FStatConstants::RAW_SecondsPerCycle)
		{
			// SecondsPerCycle may vary over time, so we update it here
			MillisecondsPerCycle = StatMessage.GetValue_double() * 1000.0f;
		}

		for (int32 Jndex = 0; Jndex < StatRawNames.Num(); ++Jndex)
		{
			const FName StatRawName = StatMessage.NameAndInfo.GetRawName();
			if (StatRawName == StatRawNames[Jndex])
			{
				FString StatValue;
				if (StatMessage.NameAndInfo.GetFlag( EStatMetaFlags::IsPackedCCAndDuration ))
				{
					StatValue = LexToString(MillisecondsPerCycle * FromPackedCallCountDuration_Duration( StatMessage.GetValue_int64() ));
				}
				else if (StatMessage.NameAndInfo.GetFlag( EStatMetaFlags::IsCycle ))
				{
					StatValue = LexToString(MillisecondsPerCycle * StatMessage.GetValue_int64());
				}
				else
				{
					switch (StatMessage.NameAndInfo.GetField<EStatDataType>())
					{
					case EStatDataType::ST_double: StatValue = LexToString(StatMessage.GetValue_double()); break;
					case EStatDataType::ST_int64: StatValue = LexToString(StatMessage.GetValue_int64()); break;
					case EStatDataType::ST_FName: StatValue = StatMessage.GetValue_FName().ToString(); break;
					default: StatValue = TEXT("<unknown type>");
					}
				}

				// write out to the csv file
				WriteString( TEXT("%d,%s,%s\r\n"), Frame, *StatShortNames[Jndex].ToString(), *StatValue );
			}
		}
	}
}

void FCSVStatsProfiler::WriteString( const TCHAR* Format, ... )
{
	TCHAR Array[1024];
	va_list ArgPtr;
	va_start( ArgPtr, Format );
	// Build the string.
	int32 Result = FCString::GetVarArgs( Array, UE_ARRAY_COUNT( Array ), Format, ArgPtr );
	va_end(ArgPtr);

	// Now write that to the file.
	CSVWriter->Serialize( (void*)Array, Result * sizeof(TCHAR) );
}

void FStatsConvertCommand::InternalRun()
{
	// Get the target file.
	FString TargetFile;
	FParse::Value(FCommandLine::Get(), TEXT("-INFILE="), TargetFile);

	FString OutFile;
	FParse::Value(FCommandLine::Get(), TEXT("-OUTFILE="), OutFile);

	// Stat list can contains only stat's FName.
	FString StatListString;
	FParse::Value(FCommandLine::Get(), TEXT("-STATLIST="), StatListString);

	// Open a CSV file for write.
	TUniquePtr<FArchive> FileWriter( IFileManager::Get().CreateFileWriter( *OutFile ) );
	if (!FileWriter)
	{
		UE_LOGF( LogStats, Error, "Could not open output file: %ls", *OutFile );
		return;
	}

	TUniquePtr<FCSVStatsProfiler> Instance( FStatsReader<FCSVStatsProfiler>::Create( *TargetFile ) );
	if (Instance)
	{
		TArray<FName> StatList;

		// Get the list of stats.
		TArray<FString> StatArrayString;
		if (StatListString.ParseIntoArray( StatArrayString, TEXT( "+" ), true ) == 0)
		{
			StatArrayString.Add( TEXT( "STAT_FrameTime" ) );
		}

		Instance->Initialize( FileWriter.Get(), StatArrayString );
		Instance->ReadAndProcessSynchronously();

		while (Instance->IsBusy())
		{
			FPlatformProcess::Sleep( 2.0f );
			UE_LOGF( LogStats, Log, "FStatsConvertCommand: Stage: %ls / %3i%%", *Instance->GetProcessingStageAsString(), Instance->GetStageProgress() );
		}
	}
}

#endif // STATS && UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
