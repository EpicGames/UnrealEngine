// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/StatsData.h"
#include "Stats/StatsFile.h"

#if STATS && UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8

class FStatsConvertCommand
{
public:

	/** Executes the command. */
	static void Run()
	{
		FStatsConvertCommand Instance;
		Instance.InternalRun();
	}

protected:

	void InternalRun();

};

#endif // STATS && UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8