// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterfaceCommand.h"
#include "IAutomationControllerModule.h"
#include "ISlateReflectorModule.h"
#include "Interfaces/IPluginManager.h"
#include "StandaloneRenderer.h"
#include "Async/TaskGraphInterfaces.h"
#include "ISourceCodeAccessModule.h"
#include "Containers/Ticker.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformApplicationMisc.h"
#include "OutputLogModule.h"
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "Stats/StatsSystem.h"

#define IDEAL_FRAMERATE 60;


namespace UserInterfaceCommand
{
	TSharedPtr<FTabManager::FLayout> ApplicationLayout;
	TSharedRef<FWorkspaceItem> DeveloperTools = FWorkspaceItem::NewGroup( NSLOCTEXT("UnrealFrontend", "DeveloperToolsMenu", "Developer Tools") );
}


/* FUserInterfaceCommand interface
 *****************************************************************************/

void FUserInterfaceCommand::Run(  )
{
	FString UnrealFrontendLayoutIni = FPaths::GetPath(GEngineIni) + "/Layout.ini";

	// ensure target platform manager is referenced early as it must be created on the main thread
	FModuleManager::Get().LoadModuleChecked("DesktopPlatform");
	FConfigCacheIni::InitializeConfigSystem();

	GetTargetPlatformManager();

#if !WITH_EDITOR
	// some items in Launcher Services need the editor ini settings - make sure they're loaded if they've not been done already
	FConfigContext Context = FConfigContext::ReadIntoGConfig(); 
	Context.Load(TEXT("Editor"), GEditorIni);
#endif

	// Crank up a normal Slate application using the platform's standalone renderer.
	FSlateApplication::InitializeAsStandaloneApplication(GetStandardStandaloneRenderer());

	// load required modules
	FModuleManager::Get().LoadModuleChecked("Messaging");
	FModuleManager::Get().LoadModuleChecked("OutputLog");

	IAutomationControllerModule& AutomationControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>("AutomationController");
	AutomationControllerModule.Init();

	// load plug-ins
	// @todo: allow for better plug-in support in standalone Slate applications
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreDefault);

	// load optional modules
	FModuleManager::Get().LoadModule("DeviceManager");
#if STATS && defined(UE_DEPRECATED_PROFILER_ENABLED) && UE_DEPRECATED_PROFILER_ENABLED
	FModuleManager::Get().LoadModule("ProfilerClient");
#endif
	FModuleManager::Get().LoadModule("LegacyProjectLauncher");
	FModuleManager::Get().LoadModule("SessionFrontend");
	FModuleManager::Get().LoadModule("SettingsEditor");
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::Default);

	InitializeSlateApplication(UnrealFrontendLayoutIni);

	// initialize source code access
	// Load the source code access module
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>( FName( "SourceCodeAccess" ) );

	// Initialize directory watcher module
	// Load the directory watcher module
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(FName("DirectoryWatcher"));

	// trigger loading of post default plug-ins
	// (for UDP Messaging)
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PostDefault);

	// Manually load in the source code access plugins, as standalone programs don't currently support plugins.
#if PLATFORM_MAC
	IModuleInterface& XCodeSourceCodeAccessModule = FModuleManager::LoadModuleChecked<IModuleInterface>( FName( "XCodeSourceCodeAccess" ) );
	SourceCodeAccessModule.SetAccessor(FName("XCodeSourceCodeAccess"));
#elif PLATFORM_WINDOWS
	IModuleInterface& VisualStudioSourceCodeAccessModule = FModuleManager::LoadModuleChecked<IModuleInterface>( FName( "VisualStudioSourceCodeAccess" ) );
	SourceCodeAccessModule.SetAccessor(FName("VisualStudioSourceCodeAccess"));
#endif

	// enter main loop
	double DeltaTime = 0.0;
	double LastTime = FPlatformTime::Seconds();
	const float IdealFrameTime = 1.0f / IDEAL_FRAMERATE;

	while (!IsEngineExitRequested())
	{
		//Save the state of the tabs here rather than after close of application (the tabs are undesirably saved out with ClosedTab state on application close)
		//UserInterfaceCommand::UserConfiguredNewLayout = FGlobalTabmanager::Get()->PersistLayout();

		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

		//We have to force tick here to be able to update Screen Comparison tab in the Slate applications without Engine loop
		if (!FApp::IsProjectNameEmpty())
		{
			DirectoryWatcherModule.Get()->Tick(FApp::GetDeltaTime());
		}

		FSlateApplication::Get().PumpMessages();
		FSlateApplication::Get().Tick();
		FTSTicker::GetCoreTicker().Tick(DeltaTime);
		AutomationControllerModule.Tick();

		// throttle frame rate
		FPlatformProcess::Sleep(FMath::Max<float>(0.0f, IdealFrameTime - (FPlatformTime::Seconds() - LastTime)));

		double CurrentTime = FPlatformTime::Seconds();
		DeltaTime =  CurrentTime - LastTime;
		LastTime = CurrentTime;

		UE::Stats::FStats::AdvanceFrame( false );
		FCoreDelegates::OnEndFrame.Broadcast();

		GLog->FlushThreadedLogs();
	}

	ShutdownSlateApplication(UnrealFrontendLayoutIni);
}


/* FUserInterfaceCommand implementation
 *****************************************************************************/

void FUserInterfaceCommand::InitializeSlateApplication( const FString& LayoutIni )
{
	FSlateApplication::InitHighDPI(true);

	FGlobalTabmanager::Get()->SetApplicationTitle(NSLOCTEXT("UnrealFrontend", "AppTitle", "Unreal Frontend"));

	// load widget reflector
	const bool bAllowDebugTools = FParse::Param(FCommandLine::Get(), TEXT("DebugTools"));

	if (bAllowDebugTools)
	{
		ISlateReflectorModule* SlateReflectorModule = FModuleManager::LoadModulePtr<ISlateReflectorModule>("SlateReflector");
		if (SlateReflectorModule)
		{
			SlateReflectorModule->RegisterTabSpawner(UserInterfaceCommand::DeveloperTools);
		}
	}

	const float DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(10.0f, 10.0f);

	// restore application layout
	TSharedRef<FTabManager::FLayout> NewLayout = FTabManager::NewLayout("SessionFrontendLayout_v1.3")
		->AddArea
		(
			FTabManager::NewArea(1280.f * DPIScaleFactor, 720.0f * DPIScaleFactor)
				->Split
				(
					FTabManager::NewStack()
						->AddTab(FName("DeviceOutputLog"), ETabState::OpenedTab)
						->AddTab(FName("DeviceManager"), ETabState::OpenedTab)
						->AddTab(FName("MessagingDebugger"), ETabState::ClosedTab)
						->AddTab(FName("SessionFrontend"), ETabState::OpenedTab)
						->AddTab(FName("ProjectLauncher"), ETabState::OpenedTab)
						->AddTab(FName("LegacyProjectLauncher"), ETabState::ClosedTab)
				)
		)
		->AddArea
		(
			FTabManager::NewArea(600.0f * DPIScaleFactor, 600.0f * DPIScaleFactor)
				->SetWindow(FVector2D(10.0f * DPIScaleFactor, 10.0f * DPIScaleFactor), false)
				->Split
				(
					FTabManager::NewStack()->AddTab("WidgetReflector", bAllowDebugTools ? ETabState::OpenedTab : ETabState::ClosedTab)
				)
		);

	// Load layout from ini file.
	UserInterfaceCommand::ApplicationLayout = FLayoutSaveRestore::LoadFromConfig(LayoutIni, NewLayout);
	// Restore application layout.
	const bool bEmbedTitleAreaContent = false;
	const EOutputCanBeNullptr OutputCanBeNullptr = EOutputCanBeNullptr::IfNoTabValid;
	FGlobalTabmanager::Get()->RestoreFrom(UserInterfaceCommand::ApplicationLayout.ToSharedRef(), TSharedPtr<SWindow>(), bEmbedTitleAreaContent, OutputCanBeNullptr);
}

void FUserInterfaceCommand::ShutdownSlateApplication( const FString& LayoutIni )
{
	check(UserInterfaceCommand::ApplicationLayout.IsValid());

	// save application layout
	FLayoutSaveRestore::SaveToConfig(LayoutIni, UserInterfaceCommand::ApplicationLayout.ToSharedRef());
	GConfig->Flush(false, LayoutIni);

	// shut down application
	FSlateApplication::Shutdown();
}
