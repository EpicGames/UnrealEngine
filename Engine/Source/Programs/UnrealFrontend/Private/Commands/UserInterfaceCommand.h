// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FUserInterfaceCommand
{
public:

	/** Executes the command. */
	static void Run( );

protected:

	/**
	 * Initializes the Slate application.
	 *
	 * @param LayoutInit The path to the layout configuration file.
	 */
	static void InitializeSlateApplication( const FString& LayoutIni );

	/**
	 * Shuts down the Slate application.
	 *
	 * @param LayoutInit The path to the layout configuration file.
	 */
	static void ShutdownSlateApplication( const FString& LayoutIni );
};
