// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealFrontendMain.h"
#include "RequiredProgramMainCPPInclude.h"

#include "StatsConvertCommand.h"
#include "UserInterfaceCommand.h"
#include "LaunchFromProfileCommand.h"
#include "ExecuteConsoleCommand.h"

DEFINE_LOG_CATEGORY(LogUFECommands);

IMPLEMENT_APPLICATION(UnrealFrontend, "UnrealFrontend");

/**
 * Platform agnostic implementation of the main entry point.
 */
int32 UnrealFrontendMain( const TCHAR* CommandLine )
{
	FTaskTagScope TaskTagScope(ETaskTag::EGameThread);

	// Override the stack size for the thread pool.
	FQueuedThreadPool::OverrideStackSize = 256 * 1024;

	FCommandLine::Set(CommandLine);

	FString Command;
	FString Params;
	FString NewCommandLine = CommandLine;

	// process command line parameters
	bool bRunCommand = FParse::Value(*NewCommandLine, TEXT("-RUN="), Command);
	
	if (Command.IsEmpty())
	{
		GLog->Logf(ELogVerbosity::Warning, TEXT("The command line argument '-RUN=' does not have a command name associated with it."));
	}

	if (bRunCommand)
	{
		// extract off any '-PARAM=' parameters so that they aren't accidentally parsed by engine init
		FParse::Value(*NewCommandLine, TEXT("-PARAMS="), Params);

		if (Params.Len() > 0)
		{
			// remove from the command line & trim quotes
			NewCommandLine = NewCommandLine.Replace(*Params, TEXT(""));
			Params = Params.TrimQuotes();
		}
	}

	// Add -Messaging if it was not given in the command line.
	if (!FParse::Param(*NewCommandLine, TEXT("MESSAGING")))
	{
		NewCommandLine += TEXT(" -Messaging");
	}

	// Add '-Log' if the Frontend was run with '-RUN=' without '-LOG' so we can read any potential log output.
	if (bRunCommand && !FParse::Param(*NewCommandLine, TEXT("LOG")))
	{
		NewCommandLine += TEXT(" -Log");
	}

	// initialize core
	GEngineLoop.PreInit(*NewCommandLine);
	FModuleManager::Get().StartProcessingNewlyLoadedObjects();

	bool Succeeded = true;

	// Execute desired command
	// To execute, run with '-RUN="COMMAND_NAME_FOUND_BELOW"'. 
	// NOTE - Some commands may require extra command line parameters.
	if (bRunCommand)
	{
#if STATS && UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
		if (Command.Equals(TEXT("CONVERT"), ESearchCase::IgnoreCase))
		{
			FStatsConvertCommand::Run();
		}
		else
#endif // STATS && UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
		// The 'LAUNCHPROFILE' command also needs '-PROFILENAME="MY_PROFILE_NAME"' as a command line parameter.
		if (Command.Equals(TEXT("LAUNCHPROFILE"), ESearchCase::IgnoreCase))
		{
			FLaunchFromProfileCommand* ProfileLaunch = new FLaunchFromProfileCommand;
			ProfileLaunch->Run(Params);
		}
		else if (Command.Equals(TEXT("CONSOLE"), ESearchCase::IgnoreCase))
		{
			FExecuteConsoleCommand* ConsoleCommand = new FExecuteConsoleCommand;
			ConsoleCommand->Run(Params);
		}
	}
	else
	{
		FUserInterfaceCommand::Run();
	}

	// shut down
	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();

	return Succeeded ? 0 : -1;
}
