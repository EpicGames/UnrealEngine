// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using UnrealBuildTool;

public class UnrealFrontend : ModuleRules
{
	public UnrealFrontend(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		bRequiresPlatformSDK = true;

		PrivateIncludePaths.AddRange(
			new string[] {
				"Programs/UnrealFrontend/Private/Commands",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"AutomationController",
				"Core",
				"CoreUObject",
				"DeviceManager",
				"LauncherServices",
				"Messaging",
				"OutputLog",
				"LegacyProjectLauncher",
				"Projects",
				"SessionFrontend",
				"SessionServices",
				"Slate",
				"SlateCore",
				"SourceCodeAccess",
				"StandaloneRenderer",
				"TargetDeviceServices",
				"TargetPlatform",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"SlateReflector",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"SlateReflector"
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PrivateDependencyModuleNames.Add("XCodeSourceCodeAccess");
		}
		else if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDependencyModuleNames.Add("VisualStudioSourceCodeAccess");
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"UnixCommonStartup"
				}
			);
		}

		// @todo: allow for better plug-in support in standalone Slate apps
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DirectoryWatcher",
				"Networking",
				"Sockets",
				"TcpMessaging",
				"UdpMessaging",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Messaging",
			}
		);

		if (Target.GlobalDefinitions.Contains("UE_DEPRECATED_PROFILER_ENABLED=1"))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Profiler",
					"ProfilerClient",
				}
			);
		}
	}
}
