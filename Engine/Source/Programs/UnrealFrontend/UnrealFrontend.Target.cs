// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealFrontendTarget : TargetRules
{
	public UnrealFrontendTarget( TargetInfo Target ) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Modular;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		AdditionalPlugins.Add("UdpMessaging");
		LaunchModuleName = "UnrealFrontend";
		
		// Stats are required even in Shipping
		GlobalDefinitions.Add("FORCE_USE_STATS=1");

		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = true;
		bForceBuildTargetPlatforms = true;
		bCompileWithStatsWithoutEngine = true;
		bCompileWithPluginSupport = true;

		// For UI functionality
		bBuildDeveloperTools = true;

		bHasExports = false;

		bEnableTrace = true;

		// Old Profiler (SessionFrontend/Profiler) is deprecated since UE 5.0. Use Trace/UnrealInsights instead.
		//GlobalDefinitions.Add("UE_DEPRECATED_PROFILER_ENABLED=1");

		// new project launcher
		AdditionalPlugins.Add("ProjectLauncher");
	}
}
